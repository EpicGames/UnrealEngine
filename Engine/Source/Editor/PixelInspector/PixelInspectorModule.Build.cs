// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PixelInspectorModule : ModuleRules
{
    public PixelInspectorModule(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange
        (
            new string[] {
				"Core",
                "InputCore",
                "CoreUObject",
				"RHI",
				"RenderCore",
				"Slate",
                "Engine",
                "UnrealEd",
                "PropertyEditor",
			}
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Renderer",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"LevelEditor"
			}
		);

		if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
    				"SlateCore",
                }
            );

            CircularlyReferencedDependentModules.AddRange(
                new string[] {
                    "UnrealEd"
                }
            );
        }
	}
}
