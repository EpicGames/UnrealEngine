// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspector.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/CoreStyle.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelInspectorView.h"
#include "PixelInspectorStyle.h"
#include "ScreenPass.h"
#include "UnrealClient.h"
#include "PostProcess/PostProcessMaterialInputs.h"

#include "EngineGlobals.h"
#include "EditorViewportClient.h"
#include "Editor.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "LevelEditor.h"
#include "TextureResource.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "SceneView.h"
#include "RenderGraph.h"
#include "HAL/IConsoleManager.h"
#include "SceneViewExtension.h"
#include "SceneInterface.h"
#include "Modules/ModuleManager.h"

#define PIXEL_INSPECTOR_REQUEST_TIMEOUT 10
#define MINIMUM_TICK_BETWEEN_CREATE_REQUEST 10
#define DEFAULT_DISPLAY_GAMMA 2.2f
#define LOCTEXT_NAMESPACE "PixelInspector"


namespace PixelInspector
{
	SPixelInspector::SPixelInspector()
	{
		DisplayResult = nullptr;
		LastViewportInspectionPosition = FIntPoint(-1, -1);
		LastViewportId = 0;
		LastViewportInspectionSize = FIntPoint(1, 1);

		Buffer_FinalColor_AnyFormat[0] = nullptr;
		Buffer_FinalColor_AnyFormat[1] = nullptr;
		Buffer_SceneColorBeforePost_Float[0] = nullptr;
		Buffer_SceneColorBeforePost_Float[1] = nullptr;
		Buffer_SceneColorBeforeToneMap_Float[0] = nullptr;
		Buffer_SceneColorBeforeToneMap_Float[1] = nullptr;
		Buffer_Depth_Float[0] = nullptr;
		Buffer_Depth_Float[1] = nullptr;
		Buffer_A_Float[0] = nullptr;
		Buffer_A_Float[1] = nullptr;
		Buffer_A_RGB8[0] = nullptr;
		Buffer_A_RGB8[1] = nullptr;
		Buffer_A_RGB10[0] = nullptr;
		Buffer_A_RGB10[1] = nullptr;
		Buffer_BCDEF_Float[0] = nullptr;
		Buffer_BCDEF_Float[1] = nullptr;
		Buffer_BCDEF_RGB8[0] = nullptr;
		Buffer_BCDEF_RGB8[1] = nullptr;

		TickSinceLastCreateRequest = 0;
		LastBufferIndex = 0;

		OnLevelActorDeletedDelegateHandle = GEngine->OnLevelActorDeleted().AddRaw(this, &SPixelInspector::OnLevelActorDeleted);
		OnEditorCloseHandle = GEditor->OnEditorClose().AddRaw(this, &SPixelInspector::ReleaseRessource);

		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(FName(TEXT("LevelEditor")));
		OnRedrawViewportHandle = LevelEditor.OnRedrawLevelEditingViewports().AddRaw(this, &SPixelInspector::OnRedrawViewport);

		OnApplicationPreInputKeyDownListenerHandle = FSlateApplication::Get().OnApplicationPreInputKeyDownListener().AddRaw(this, &SPixelInspector::OnApplicationPreInputKeyDownListener);
		
		SetPixelInspectorState(false);
	}

	SPixelInspector::~SPixelInspector()
	{
		ReleaseRessource();
	}
	
	void SPixelInspector::OnApplicationPreInputKeyDownListener(const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape && (bIsPixelInspectorEnable))
		{
			// disable the pixel inspector
			SetPixelInspectorState(false);
		}
	}
	
	void SPixelInspector::OnWindowClosed()
	{
		if(bIsPixelInspectorEnable)
		{
			SetPixelInspectorState(false);

			// We need to invalide the draw as the pixel inspector message is left on from the last draw
			GEditor->RedrawLevelEditingViewports();
		}
	}
	
	void SPixelInspector::ReleaseRessource()
	{
		SetPixelInspectorState(false);

		if (DisplayResult != nullptr)
		{
			DisplayResult->RemoveFromRoot();
			DisplayResult->ClearFlags(RF_Standalone);
			DisplayResult = nullptr;
		}

		ReleaseAllRequests();

		if (OnLevelActorDeletedDelegateHandle.IsValid())
		{
			GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedDelegateHandle);
			OnLevelActorDeletedDelegateHandle = FDelegateHandle();
		}

		if (OnEditorCloseHandle.IsValid())
		{
			GEditor->OnEditorClose().Remove(OnEditorCloseHandle);
			OnEditorCloseHandle = FDelegateHandle();
		}

		if (OnRedrawViewportHandle.IsValid())
		{
			FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(FName(TEXT("LevelEditor")));
			LevelEditor.OnRedrawLevelEditingViewports().Remove(OnRedrawViewportHandle);
			OnRedrawViewportHandle = FDelegateHandle();
		}

		if (OnApplicationPreInputKeyDownListenerHandle.IsValid())
		{
			FSlateApplication::Get().OnApplicationPreInputKeyDownListener().Remove(OnApplicationPreInputKeyDownListenerHandle);
			OnApplicationPreInputKeyDownListenerHandle = FDelegateHandle();
		}

		if (DisplayDetailsView.IsValid())
		{
			DisplayDetailsView->SetObject(nullptr);
			DisplayDetailsView = nullptr;
		}
	}

	void SPixelInspector::ReleaseAllRequests()
	{
		//Clear all pending requests because buffer will be clear by the graphic
		for (int i = 0; i < 2; ++i)
		{
			Requests[i].RenderingCommandSend = true;
			Requests[i].RequestComplete = true;
			ReleaseBuffers(i);
		}
		if (DisplayResult != nullptr)
		{
			DisplayResult->RemoveFromRoot();
			DisplayResult->ClearFlags(RF_Standalone);
			DisplayResult = nullptr;
		}
	}

	void SPixelInspector::OnLevelActorDeleted(AActor* Actor)
	{
		ReleaseAllRequests();
	}
	
	void SPixelInspector::OnRedrawViewport(bool bInvalidateHitProxies)
	{
		ReleaseAllRequests();
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void SPixelInspector::Construct(const FArguments& InArgs)
	{
		//Set the LastViewportId to point on the active viewport
		FViewport *ActiveViewport = GEditor->GetActiveViewport();
		for (FEditorViewportClient *EditorViewport : GEditor->GetAllViewportClients())
		{
			if (ActiveViewport == EditorViewport->Viewport && EditorViewport->ViewState.GetReference() != nullptr)
			{
				LastViewportId = EditorViewport->ViewState.GetReference()->GetViewKey();
			}
		}

		TSharedPtr<SBox> InspectorBox;
		//Create the PixelInspector UI
		TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 0.0f, 3.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ToolTipText(this, &SPixelInspector::GetPixelInspectorEnableButtonTooltipText)
				.OnClicked(this, &SPixelInspector::HandleTogglePixelInspectorEnableButton)
				[
					SNew(SImage)
					.Image(this, &SPixelInspector::GetPixelInspectorEnableButtonBrush)
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(6.0f, 3.0f, 0.0f, 3.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.MinDesiredWidth(75)
				.Text(this, &SPixelInspector::GetPixelInspectorEnableButtonText)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 16.0f, 3.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.MinDesiredWidth(75)
				.Text(LOCTEXT("PixelInspector_ViewportIdValue", "Viewport Id"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 3.0f)
			[
				SNew(SNumericEntryBox<uint32>)
				.IsEnabled(false)
				.MinDesiredValueWidth(75)
				.Value(this, &SPixelInspector::GetCurrentViewportId)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 16.0f, 3.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.MinDesiredWidth(75)
				.Text(LOCTEXT("PixelInspector_ViewportCoordinate", "Coordinate"))
				.ToolTipText(LOCTEXT("PixelInspector_ViewportCoordinateTooltip", "Coordinate relative to the inspected viewport"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 8.0f, 3.0f)
			[
				SNew(SNumericEntryBox<int32>)
				.IsEnabled(this, &SPixelInspector::IsPixelInspectorEnable)
				.Value(this, &SPixelInspector::GetCurrentCoordinateX)
				.OnValueChanged(this, &SPixelInspector::SetCurrentCoordinateX)
				.OnValueCommitted(this, &SPixelInspector::SetCurrentCoordinateXCommit)
				.AllowSpin(true)
				.MinValue(0)
				.MaxSliderValue(this, &SPixelInspector::GetMaxCoordinateX)
				.MinDesiredValueWidth(75)
				.Label()
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CoordinateViewport_X", "X"))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 3.0f, 8.0f, 3.0f)
			.AutoWidth()
			[
				SNew(SNumericEntryBox<int32>)
				.IsEnabled(this, &SPixelInspector::IsPixelInspectorEnable)
				.Value(this, &SPixelInspector::GetCurrentCoordinateY)
				.OnValueChanged(this, &SPixelInspector::SetCurrentCoordinateY)
				.OnValueCommitted(this, &SPixelInspector::SetCurrentCoordinateYCommit)
				.AllowSpin(true)
				.MinValue(0)
				.MaxSliderValue(this, &SPixelInspector::GetMaxCoordinateY)
				.MinDesiredValueWidth(75)
				.Label()
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CoordinateViewport_Y", "Y"))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 12.0f, 0.0f, 3.0f)
		.FillHeight(1.0f)
		[
			SAssignNew(InspectorBox, SBox)
		];

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bSearchInitialKeyFocus = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DisplayDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		InspectorBox->SetContent(DisplayDetailsView->AsShared());
		//Create a property Detail view
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SOverlay)

				// Overlay slot for the main HLOD window area
				+ SOverlay::Slot()
				[
					VerticalBox.ToSharedRef()
				]
			]
		];
		
	}

	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	FReply SPixelInspector::HandleTogglePixelInspectorEnableButton()
	{
		SetPixelInspectorState(!bIsPixelInspectorEnable);
		if (bIsPixelInspectorEnable)
		{
			if (LastViewportInspectionPosition == FIntPoint(-1, -1))
			{
				//Let the system inspect a pixel so the user can see the UI appear
				LastViewportInspectionPosition = FIntPoint(0, 0);
			}
			//Make sure the viewport is switch to realtime
			SetCurrentViewportInRealtime();
		}
		return FReply::Handled();
	}

	FText SPixelInspector::GetPixelInspectorEnableButtonText() const
	{
		if (bIsPixelInspectorEnable)
		{
			return LOCTEXT("PixelInspector_EnableCheckbox_Inspecting", "Inspecting");
		}

		return LOCTEXT("PixelInspectorMouseHover_EnableCheckbox", "Start Pixel Inspector");
	}

	FText SPixelInspector::GetPixelInspectorEnableButtonTooltipText() const
	{
		if (bIsPixelInspectorEnable)
		{
			return LOCTEXT("PixelInspector_EnableCheckbox_ESC", "Inspecting (ESC to stop)");
		}

		return LOCTEXT("PixelInspectorMouseHover_EnableCheckbox", "Start Pixel Inspector");
	}

	const FSlateBrush* SPixelInspector::GetPixelInspectorEnableButtonBrush() const
	{
		return bIsPixelInspectorEnable ? FPixelInspectorStyle::Get()->GetBrush("PixelInspector.Enabled") : FPixelInspectorStyle::Get()->GetBrush("PixelInspector.Disabled");
	}

	void SPixelInspector::SetCurrentCoordinateXCommit(int32 NewValue, ETextCommit::Type)
	{
		ReleaseAllRequests();
		SetCurrentCoordinateX(NewValue);
	}

	void SPixelInspector::SetCurrentCoordinateX(int32 NewValue)
	{
		LastViewportInspectionPosition.X = NewValue;
	}

	void SPixelInspector::SetCurrentCoordinateYCommit(int32 NewValue, ETextCommit::Type)
	{
		ReleaseAllRequests();
		SetCurrentCoordinateY(NewValue);
	}
	void SPixelInspector::SetCurrentCoordinateY(int32 NewValue)
	{
		LastViewportInspectionPosition.Y = NewValue;
	}

	void SPixelInspector::SetCurrentCoordinate(FIntPoint NewCoordinate, bool ReleaseAllRequest)
	{
		if (ReleaseAllRequest)
		{
			ReleaseAllRequests();
		}
		LastViewportInspectionPosition.X = NewCoordinate.X;
		LastViewportInspectionPosition.Y = NewCoordinate.Y;
	}

	TOptional<int32> SPixelInspector::GetMaxCoordinateX() const
	{
		return LastViewportInspectionSize.X - 1;
	}

	TOptional<int32> SPixelInspector::GetMaxCoordinateY() const
	{
		return LastViewportInspectionSize.Y - 1;
	}

	void SPixelInspector::SetCurrentViewportInRealtime()
	{
		//Force viewport refresh
		for (FEditorViewportClient *EditorViewport : GEditor->GetAllViewportClients())
		{
			if (EditorViewport->ViewState.GetReference() != nullptr)
			{
				if (EditorViewport->ViewState.GetReference()->GetViewKey() == LastViewportId)
				{
					if (!EditorViewport->IsRealtime())
					{
						const bool bShouldBeRealtime = true;
						EditorViewport->AddRealtimeOverride(bShouldBeRealtime, LOCTEXT("RealtimeOverrideMessage_PixelInspector", "Pixel Inspector"));
					}
				}
			}
		}
	}

	void SPixelInspector::SetPixelInspectorState(bool bInIsPixelInspectorEnabled)
	{
		bIsPixelInspectorEnable = bInIsPixelInspectorEnabled;
		if (bIsPixelInspectorEnable)
		{
			if (!PixelInspectorSceneViewExtension.IsValid())
			{
				// create new scene view extension
				PixelInspectorSceneViewExtension = FSceneViewExtensions::NewExtension<FPixelInspectorSceneViewExtension>();
			}
		}
		else
		{
			if (PixelInspectorSceneViewExtension.IsValid())
			{
				// Release scene view extension as we no longer need it.
				PixelInspectorSceneViewExtension.Reset();
				PixelInspectorSceneViewExtension = nullptr;
			}
		}
	}

	void SPixelInspector::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		TickSinceLastCreateRequest++;
	}
	
	void SPixelInspector::CreatePixelInspectorRequest(FVector2D InspectViewportUV, int32 viewportUniqueId, FSceneInterface *SceneInterface, bool bInGameViewMode, float InPreExposure)
	{
		if (TickSinceLastCreateRequest < MINIMUM_TICK_BETWEEN_CREATE_REQUEST)
			return;

		//Make sure we dont get value outside the viewport size
		if ( InspectViewportUV.X >= 1.0f || InspectViewportUV.Y >= 1.0f || InspectViewportUV.X <= 0.0f || InspectViewportUV.Y <= 0.0f )
		{
			return;
		}

		TickSinceLastCreateRequest = 0;
		// We need to know if the GBuffer is in low, default or high precision buffer
		const auto CVarGBufferFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GBufferFormat"));
		//0: lower precision (8bit per component, for profiling)
		//1: low precision (default)
		//5: high precision
		const int32 GBufferFormat = CVarGBufferFormat != nullptr ? CVarGBufferFormat->GetValueOnGameThread() : 1;

		const bool AllowStaticLighting = IsStaticLightingAllowed();
		
		//Try to create the request buffer
		int32 BufferIndex = CreateRequestBuffer(SceneInterface, GBufferFormat, bInGameViewMode);
		if (BufferIndex == -1)
			return;
		
		Requests[BufferIndex].SetRequestData(FVector2f(InspectViewportUV), BufferIndex, viewportUniqueId, GBufferFormat, AllowStaticLighting, InPreExposure);	// LWC_TODO: Precision loss
		SceneInterface->AddPixelInspectorRequest(&(Requests[BufferIndex]));
	}

	void SPixelInspector::ReleaseBuffers(int32 BufferIndex)
	{
		check(BufferIndex >= 0 && BufferIndex < 2);
		if (Buffer_FinalColor_AnyFormat[BufferIndex] != nullptr)
		{
			Buffer_FinalColor_AnyFormat[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_FinalColor_AnyFormat[BufferIndex]->RemoveFromRoot();
			Buffer_FinalColor_AnyFormat[BufferIndex] = nullptr;
		}
		if (Buffer_SceneColorBeforePost_Float[BufferIndex] != nullptr)
		{
			Buffer_SceneColorBeforePost_Float[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_SceneColorBeforePost_Float[BufferIndex]->RemoveFromRoot();
			Buffer_SceneColorBeforePost_Float[BufferIndex] = nullptr;
		}
		if (Buffer_SceneColorBeforeToneMap_Float[BufferIndex] != nullptr)
		{
			Buffer_SceneColorBeforeToneMap_Float[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_SceneColorBeforeToneMap_Float[BufferIndex]->RemoveFromRoot();
			Buffer_SceneColorBeforeToneMap_Float[BufferIndex] = nullptr;
		}
		if (Buffer_Depth_Float[BufferIndex] != nullptr)
		{
			Buffer_Depth_Float[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_Depth_Float[BufferIndex]->RemoveFromRoot();
			Buffer_Depth_Float[BufferIndex] = nullptr;
		}
		if (Buffer_A_Float[BufferIndex] != nullptr)
		{
			Buffer_A_Float[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_A_Float[BufferIndex]->RemoveFromRoot();
			Buffer_A_Float[BufferIndex] = nullptr;
		}
		if (Buffer_A_RGB8[BufferIndex] != nullptr)
		{
			Buffer_A_RGB8[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_A_RGB8[BufferIndex]->RemoveFromRoot();
			Buffer_A_RGB8[BufferIndex] = nullptr;
		}
		if (Buffer_A_RGB10[BufferIndex] != nullptr)
		{
			Buffer_A_RGB10[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_A_RGB10[BufferIndex]->RemoveFromRoot();
			Buffer_A_RGB10[BufferIndex] = nullptr;
		}
		if (Buffer_BCDEF_Float[BufferIndex] != nullptr)
		{
			Buffer_BCDEF_Float[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_BCDEF_Float[BufferIndex]->RemoveFromRoot();
			Buffer_BCDEF_Float[BufferIndex] = nullptr;
		}
		if (Buffer_BCDEF_RGB8[BufferIndex] != nullptr)
		{
			Buffer_BCDEF_RGB8[BufferIndex]->ClearFlags(RF_Standalone);
			Buffer_BCDEF_RGB8[BufferIndex]->RemoveFromRoot();
			Buffer_BCDEF_RGB8[BufferIndex] = nullptr;
		}
	}

	int32 SPixelInspector::CreateRequestBuffer(FSceneInterface *SceneInterface, const int32 GBufferFormat, bool bInGameViewMode)
	{
		//Toggle the last buffer Index
		LastBufferIndex = (LastBufferIndex + 1) % 2;
		
		//Check if we have an available request
		if (Requests[LastBufferIndex].RequestComplete == false)
		{
			//Put back the last buffer position
			LastBufferIndex = (LastBufferIndex - 1) % 2;
			return -1;
		}
		
		//Release the old buffer
		ReleaseBuffers(LastBufferIndex);

		FTextureRenderTargetResource* FinalColorResource = nullptr;
		FTextureRenderTargetResource* SceneColorBeforePostResource = nullptr;
		FTextureRenderTargetResource* SceneColorBeforeToneMapResource = nullptr;
		FTextureRenderTargetResource* DepthResource = nullptr;
		FTextureRenderTargetResource* BufferAResource = nullptr;
		FTextureRenderTargetResource* BufferBCDEFResource = nullptr;
		
		//Final color can be in HDR (FloatRGBA) or RGB8 formats so we should rely on scene view extension to tell us which format is being used.
		Buffer_FinalColor_AnyFormat[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferFinalColorTarget"), RF_Standalone);
		Buffer_FinalColor_AnyFormat[LastBufferIndex]->AddToRoot();
		Buffer_FinalColor_AnyFormat[LastBufferIndex]->InitCustomFormat(FinalColorContextGridSize, FinalColorContextGridSize, PixelInspectorSceneViewExtension->GetFinalColorPixelFormat(), true);
		Buffer_FinalColor_AnyFormat[LastBufferIndex]->ClearColor = FLinearColor::Black;
		Buffer_FinalColor_AnyFormat[LastBufferIndex]->UpdateResourceImmediate(true);
		FinalColorResource = Buffer_FinalColor_AnyFormat[LastBufferIndex]->GameThread_GetRenderTargetResource();

		Buffer_SceneColorBeforePost_Float[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferSceneColorBeforePostTarget"), RF_Standalone);
		Buffer_SceneColorBeforePost_Float[LastBufferIndex]->AddToRoot();
		Buffer_SceneColorBeforePost_Float[LastBufferIndex]->InitCustomFormat(1, 1, PF_FloatRGBA, true);
		Buffer_SceneColorBeforePost_Float[LastBufferIndex]->ClearColor = FLinearColor::Black;
		Buffer_SceneColorBeforePost_Float[LastBufferIndex]->UpdateResourceImmediate(true);
		SceneColorBeforePostResource = Buffer_SceneColorBeforePost_Float[LastBufferIndex]->GameThread_GetRenderTargetResource();

		Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferSceneColorBeforeTonemapTarget"), RF_Standalone);
		Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex]->AddToRoot();
		Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex]->InitCustomFormat(1, 1, PixelInspectorSceneViewExtension->GetHDRPixelFormat(), true);
		Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex]->ClearColor = FLinearColor::Black;
		Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex]->UpdateResourceImmediate(true);
		SceneColorBeforeToneMapResource = Buffer_SceneColorBeforeToneMap_Float[LastBufferIndex]->GameThread_GetRenderTargetResource();

		//TODO support Non render buffer to be able to read the depth stencil
/*		Buffer_Depth_Float[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferDepthTarget"), RF_Standalone);
		Buffer_Depth_Float[LastBufferIndex]->AddToRoot();
		Buffer_Depth_Float[LastBufferIndex]->InitCustomFormat(1, 1, PF_DepthStencil, true);
		Buffer_Depth_Float[LastBufferIndex]->ClearColor = FLinearColor::Black;
		Buffer_Depth_Float[LastBufferIndex]->UpdateResourceImmediate(true);
		DepthRenderTargetResource = Buffer_Depth_Float[LastBufferIndex]->GameThread_GetRenderTargetResource();*/


		//Low precision GBuffer
		if (GBufferFormat == EGBufferFormat::Force8BitsPerChannel)
		{
			//All buffer are PF_B8G8R8A8
			Buffer_A_RGB8[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferATarget"), RF_Standalone );
			Buffer_A_RGB8[LastBufferIndex]->AddToRoot();
			Buffer_A_RGB8[LastBufferIndex]->InitCustomFormat(1, 1, PF_B8G8R8A8, true);
			Buffer_A_RGB8[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_A_RGB8[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferAResource = Buffer_A_RGB8[LastBufferIndex]->GameThread_GetRenderTargetResource();

			Buffer_BCDEF_RGB8[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferBTarget"), RF_Standalone );
			Buffer_BCDEF_RGB8[LastBufferIndex]->AddToRoot();
			Buffer_BCDEF_RGB8[LastBufferIndex]->InitCustomFormat(4, 1, PF_B8G8R8A8, true);
			Buffer_BCDEF_RGB8[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_BCDEF_RGB8[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferBCDEFResource = Buffer_BCDEF_RGB8[LastBufferIndex]->GameThread_GetRenderTargetResource();
		}
		else if(GBufferFormat == EGBufferFormat::Default)
		{
			//Default is PF_A2B10G10R10
			Buffer_A_RGB10[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferATarget"), RF_Standalone );
			Buffer_A_RGB10[LastBufferIndex]->AddToRoot();
			Buffer_A_RGB10[LastBufferIndex]->InitCustomFormat(1, 1, PF_A2B10G10R10, true);
			Buffer_A_RGB10[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_A_RGB10[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferAResource = Buffer_A_RGB10[LastBufferIndex]->GameThread_GetRenderTargetResource();

			//Default is PF_B8G8R8A8
			Buffer_BCDEF_RGB8[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferBTarget"), RF_Standalone );
			Buffer_BCDEF_RGB8[LastBufferIndex]->AddToRoot();
			Buffer_BCDEF_RGB8[LastBufferIndex]->InitCustomFormat(4, 1, PF_B8G8R8A8, true);
			Buffer_BCDEF_RGB8[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_BCDEF_RGB8[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferBCDEFResource = Buffer_BCDEF_RGB8[LastBufferIndex]->GameThread_GetRenderTargetResource();
		}
		else if (GBufferFormat == EGBufferFormat::HighPrecisionNormals || GBufferFormat == EGBufferFormat::Force16BitsPerChannel)
		{
			//All buffer are PF_FloatRGBA
			Buffer_A_Float[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferATarget"), RF_Standalone );
			Buffer_A_Float[LastBufferIndex]->AddToRoot();
			Buffer_A_Float[LastBufferIndex]->InitCustomFormat(1, 1, PF_FloatRGBA, true);
			Buffer_A_Float[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_A_Float[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferAResource = Buffer_A_Float[LastBufferIndex]->GameThread_GetRenderTargetResource();

			Buffer_BCDEF_Float[LastBufferIndex] = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("PixelInspectorBufferBTarget"), RF_Standalone );
			Buffer_BCDEF_Float[LastBufferIndex]->AddToRoot();
			Buffer_BCDEF_Float[LastBufferIndex]->InitCustomFormat(4, 1, PF_FloatRGBA, true);
			Buffer_BCDEF_Float[LastBufferIndex]->ClearColor = FLinearColor::Black;
			Buffer_BCDEF_Float[LastBufferIndex]->UpdateResourceImmediate(true);
			BufferBCDEFResource = Buffer_BCDEF_Float[LastBufferIndex]->GameThread_GetRenderTargetResource();
		}
		else
		{
			checkf(0, TEXT("Unhandled gbuffer format (%i) during pixel inspector initializtion."), GBufferFormat);
		}	
		
		SceneInterface->InitializePixelInspector(FinalColorResource, SceneColorBeforePostResource, DepthResource, SceneColorBeforeToneMapResource, BufferAResource, BufferBCDEFResource, LastBufferIndex);

		return LastBufferIndex;
	}

	void SPixelInspector::ReadBackRequestData()
	{
		for (int RequestIndex = 0; RequestIndex < UE_ARRAY_COUNT(Requests); ++RequestIndex)
		{
			FPixelInspectorRequest& Request = Requests[RequestIndex];
			if (Request.RequestComplete == false && Request.RenderingCommandSend == true)
			{
				if (Request.FrameCountAfterRenderingCommandSend >= WAIT_FRAMENUMBER_BEFOREREADING)
				{
					if (Request.SourceViewportUV == FVector2f(-1, -1))
					{
						continue;
					}


					PixelInspectorResult PixelResult;
					PixelResult.ViewportUV = FVector2D(Request.SourceViewportUV);
					PixelResult.ViewUniqueId = Request.ViewId;
					PixelResult.PreExposure = Request.PreExposure;
					PixelResult.OneOverPreExposure = Request.PreExposure > 0.f ? (1.f / Request.PreExposure) : 1.f;;

					FTextureRenderTargetResource* RTResourceFinalColor = Buffer_FinalColor_AnyFormat[Request.BufferIndex]->GameThread_GetRenderTargetResource();
					const EPixelFormat FinalColorPixelFormat = PixelInspectorSceneViewExtension->GetFinalColorPixelFormat();
					const float Gamma = (FinalColorPixelFormat == PF_B8G8R8A8) ? 1.0f : PixelInspectorSceneViewExtension->GetGamma();
					EPixelFormatChannelFlags ValidPixelChannels = GetPixelFormatValidChannels(FinalColorPixelFormat);
					bool bHasAlphaChannel = EnumHasAnyFlags(ValidPixelChannels, EPixelFormatChannelFlags::A);

					TArray<FLinearColor> BufferFinalColorValueLinear;
					if (RTResourceFinalColor->ReadLinearColorPixels(BufferFinalColorValueLinear) == false)
					{
						BufferFinalColorValueLinear.Empty();
					}

					PixelResult.DecodeFinalColor(BufferFinalColorValueLinear, Gamma, bHasAlphaChannel);
					

					TArray<FLinearColor> BufferSceneColorValue;
					FTextureRenderTargetResource* RTResourceSceneColor = Buffer_SceneColorBeforePost_Float[Request.BufferIndex]->GameThread_GetRenderTargetResource();
					if (RTResourceSceneColor->ReadLinearColorPixels(BufferSceneColorValue) == false)
					{
						BufferSceneColorValue.Empty();
					}
					PixelResult.DecodeSceneColorBeforePostProcessing(BufferSceneColorValue);

					if (Buffer_Depth_Float[Request.BufferIndex] != nullptr)
					{
						TArray<FLinearColor> BufferDepthValue;
						FTextureRenderTargetResource* RTResourceDepth = Buffer_Depth_Float[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceDepth->ReadLinearColorPixels(BufferDepthValue) == false)
						{
							BufferDepthValue.Empty();
						}
						PixelResult.DecodeDepth(BufferDepthValue);
					}

					TArray<FLinearColor> BufferSceneColorBeforeToneMapValue;
					FTextureRenderTargetResource* RTResourceSceneColorBeforeTonemap = Buffer_SceneColorBeforeToneMap_Float[Request.BufferIndex]->GameThread_GetRenderTargetResource();
					if (RTResourceSceneColorBeforeTonemap->ReadLinearColorPixels(BufferSceneColorBeforeToneMapValue) == false)
					{
						BufferSceneColorBeforeToneMapValue.Empty();
					}

					const EPixelFormat HDRPixelFormat = PixelInspectorSceneViewExtension->GetHDRPixelFormat();
					ValidPixelChannels = GetPixelFormatValidChannels(HDRPixelFormat);
					bHasAlphaChannel = EnumHasAnyFlags(ValidPixelChannels, EPixelFormatChannelFlags::A);
					PixelResult.DecodeSceneColorBeforeToneMap(BufferSceneColorBeforeToneMapValue, bHasAlphaChannel);

					if (Request.GBufferPrecision == EGBufferFormat::Force8BitsPerChannel)
					{
						TArray<FColor> BufferAValue;
						FTextureRenderTargetResource* RTResourceA = Buffer_A_RGB8[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceA->ReadPixels(BufferAValue) == false)
						{
							BufferAValue.Empty();
						}

						TArray<FColor> BufferBCDEFValue;
						FTextureRenderTargetResource* RTResourceBCDEF = Buffer_BCDEF_RGB8[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceA->ReadPixels(BufferBCDEFValue) == false)
						{
							BufferBCDEFValue.Empty();
						}

						PixelResult.DecodeBufferData(BufferAValue, BufferBCDEFValue, Request.AllowStaticLighting);
					}
					else if (Request.GBufferPrecision == EGBufferFormat::Default)
					{
						//PF_A2B10G10R10 format is not support yet
						TArray<FLinearColor> BufferAValue;
						FTextureRenderTargetResource* RTResourceA = Buffer_A_RGB10[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceA->ReadLinearColorPixels(BufferAValue) == false)
						{
							BufferAValue.Empty();
						}

						TArray<FColor> BufferBCDEFValue;
						FTextureRenderTargetResource* RTResourceBCDEF = Buffer_BCDEF_RGB8[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceBCDEF->ReadPixels(BufferBCDEFValue) == false)
						{
							BufferBCDEFValue.Empty();
						}
						PixelResult.DecodeBufferData(BufferAValue, BufferBCDEFValue, Request.AllowStaticLighting);
					}
					else if (Request.GBufferPrecision == EGBufferFormat::HighPrecisionNormals || Request.GBufferPrecision == EGBufferFormat::Force16BitsPerChannel)
					{
						//PF_A2B10G10R10 format is not support yet
						TArray<FFloat16Color> BufferAValue;
						FTextureRenderTargetResource* RTResourceA = Buffer_A_Float[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceA->ReadFloat16Pixels(BufferAValue) == false)
						{
							BufferAValue.Empty();
						}

						TArray<FFloat16Color> BufferBCDEFValue;
						FTextureRenderTargetResource* RTResourceBCDEF = Buffer_BCDEF_Float[Request.BufferIndex]->GameThread_GetRenderTargetResource();
						if (RTResourceA->ReadFloat16Pixels(BufferBCDEFValue) == false)
						{
							BufferBCDEFValue.Empty();
						}
						PixelResult.DecodeBufferData(BufferAValue, BufferBCDEFValue, Request.AllowStaticLighting);
					}
					else
					{
						checkf(0, TEXT("Unhandled gbuffer format (%i) during pixel inspector readback."), Request.GBufferPrecision);
					}

					AccumulationResult.Add(PixelResult);
					ReleaseBuffers(RequestIndex);
					Request.RequestComplete = true;
					Request.RenderingCommandSend = true;
					Request.FrameCountAfterRenderingCommandSend = 0;
					Request.RequestTickSinceCreation = 0;
				}
				else
				{
					Requests[RequestIndex].FrameCountAfterRenderingCommandSend++;
				}
			}
			else if (Requests[RequestIndex].RequestComplete == false)
			{
				Requests[RequestIndex].RequestTickSinceCreation++;
				if (Requests[RequestIndex].RequestTickSinceCreation > PIXEL_INSPECTOR_REQUEST_TIMEOUT)
				{
					ReleaseBuffers(RequestIndex);
					Requests[RequestIndex].RequestComplete = true;
					Requests[RequestIndex].RenderingCommandSend = true;
					Requests[RequestIndex].FrameCountAfterRenderingCommandSend = 0;
					Requests[RequestIndex].RequestTickSinceCreation = 0;
				}
			}
		}
		if (AccumulationResult.Num() > 0)
		{
			if (DisplayResult == nullptr)
			{
				DisplayResult = NewObject<UPixelInspectorView>(GetTransientPackage(), FName(TEXT("PixelInspectorDisplay")), RF_Standalone);
				DisplayResult->AddToRoot();
			}
			DisplayResult->SetFromResult(AccumulationResult[0]);
			DisplayDetailsView->SetObject(DisplayResult, true);
			LastViewportInspectionPosition.X = AccumulationResult[0].ViewportUV.X * LastViewportInspectionSize.X;
			LastViewportInspectionPosition.Y = AccumulationResult[0].ViewportUV.Y * LastViewportInspectionSize.Y;
			LastViewportId = AccumulationResult[0].ViewUniqueId;
			AccumulationResult.RemoveAt(0);
		}
	}

	FPixelInspectorSceneViewExtension::FPixelInspectorSceneViewExtension(const FAutoRegister& AutoRegister)
		: FSceneViewExtensionBase(AutoRegister)
	{
		FinalColorPixelFormat = PF_B8G8R8A8;
		HDRPixelFormat = PF_FloatRGBA;
		// Default dislay gamma is hardcoded in tonemapper and is set to 2.2. 
		// We initialize this value and then get the actual value from ViewFamily.
		FinalColorGamma = DEFAULT_DISPLAY_GAMMA;
	}

	void FPixelInspectorSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
	{
		bool bAnyUsePixelInspector = false;
		for (const FSceneView* View : InViewFamily.Views)
		{
			if (View->bUsePixelInspector)
			{
				bAnyUsePixelInspector = true;
				break;
			}
		}

		if (bAnyUsePixelInspector)
		{
			const float DisplayGamma = InViewFamily.RenderTarget->GetDisplayGamma();
			// We need to apply gamma to the final color if the output is in HDR.
			FinalColorGamma = (InViewFamily.EngineShowFlags.Tonemapper == 0) ? DEFAULT_DISPLAY_GAMMA : DisplayGamma;
		}
	}

	void FPixelInspectorSceneViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
	{
		if (View.bUsePixelInspector)
		{
			if (PassId == EPostProcessingPass::FXAA)
			{
				InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FPixelInspectorSceneViewExtension::PostProcessPassAfterFxaa_RenderThread));
			}

			if (PassId == EPostProcessingPass::MotionBlur)
			{
				InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FPixelInspectorSceneViewExtension::PostProcessPassAfterMotionBlur_RenderThread));
			}
		}
	}

	FScreenPassTexture FPixelInspectorSceneViewExtension::PostProcessPassAfterFxaa_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
	{
		FinalColorPixelFormat = InOutInputs.Textures[(uint32)EPostProcessMaterialInput::SceneColor].TextureSRV->Desc.Texture->Desc.Format;
		// Don't need to modify anything, just return the untouched scene color texture back to post processing.
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
	
	FScreenPassTexture FPixelInspectorSceneViewExtension::PostProcessPassAfterMotionBlur_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
	{
		HDRPixelFormat = InOutInputs.Textures[(uint32)EPostProcessMaterialInput::SceneColor].TextureSRV->Desc.Texture->Desc.Format;
		// Don't need to modify anything, just return the untouched scene color texture back to post processing.
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

};

#undef LOCTEXT_NAMESPACE
