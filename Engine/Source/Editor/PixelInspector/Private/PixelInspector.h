// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PixelInspectorResult.h"
#include "Misc/NotifyHook.h"
#include "RendererInterface.h"
#include "SceneViewExtension.h"


class AActor;
class FSceneInterface;
class IDetailsView;
class UPixelInspectorView;
class UTextureRenderTarget2D;
struct FSlateBrush;


namespace PixelInspector
{

	class FPixelInspectorSceneViewExtension : public FSceneViewExtensionBase
	{
	public:
		FPixelInspectorSceneViewExtension(const FAutoRegister& AutoRegister);

	/** Scene View extension interface. */
	public:
		virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
		virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;

	public:
		FScreenPassTexture PostProcessPassAfterFxaa_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs);
		FScreenPassTexture PostProcessPassAfterMotionBlur_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs);
		const EPixelFormat GetFinalColorPixelFormat() const { return FinalColorPixelFormat; }
		const EPixelFormat GetHDRPixelFormat() const { return HDRPixelFormat; }
		const float GetGamma() const { return FinalColorGamma; }
		
	private:
		EPixelFormat FinalColorPixelFormat;
		EPixelFormat HDRPixelFormat;
		float FinalColorGamma;
	};


#define WAIT_FRAMENUMBER_BEFOREREADING 5
	/**
	* Implements the PixelInspector window.
	*/
	class SPixelInspector : public SCompoundWidget, public FNotifyHook
	{
	public:
		/** Default constructor. */
		SPixelInspector();

		/** Virtual destructor. */
		virtual ~SPixelInspector();

		/** Release all the ressource */
		void ReleaseRessource();

		SLATE_BEGIN_ARGS(SPixelInspector){}
		SLATE_END_ARGS()

		/**
		* Constructs this widget.
		*/
		void Construct(const FArguments& InArgs);

		//~ Begin SCompoundWidget Interface
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		//~ End SCompoundWidget Interface

		void OnApplicationPreInputKeyDownListener(const FKeyEvent& InKeyEvent);
		void OnWindowClosed();

		/** Button handlers */
		FReply HandleTogglePixelInspectorEnableButton();
		FText GetPixelInspectorEnableButtonText() const;
		FText GetPixelInspectorEnableButtonTooltipText() const;
		const FSlateBrush* GetPixelInspectorEnableButtonBrush() const;


		TOptional<uint32> GetCurrentViewportId() const { return LastViewportId; }
		FIntPoint GetCurrentCoordinate() const { return LastViewportInspectionPosition; }
		TOptional<int32> GetCurrentCoordinateX() const { return LastViewportInspectionPosition.X; }
		void SetCurrentCoordinateX(int32 NewValue);
		void SetCurrentCoordinateXCommit(int32 NewValue, ETextCommit::Type);
		TOptional<int32> GetCurrentCoordinateY() const { return LastViewportInspectionPosition.Y; }
		void SetCurrentCoordinateY(int32 NewValue);
		void SetCurrentCoordinateYCommit(int32 NewValue, ETextCommit::Type);
		TOptional<int32> GetMaxCoordinateX() const;
		TOptional<int32> GetMaxCoordinateY() const;

		/** End button handlers */

		bool IsPixelInspectorEnable() const { return bIsPixelInspectorEnable;	}

		void SetCurrentCoordinate(FIntPoint NewCoordinate, bool ReleaseAllRequest);

		void SetViewportInformation(int32 ViewportUniqueId, FIntPoint ViewportSize) { LastViewportId = ViewportUniqueId; LastViewportInspectionSize = ViewportSize; }

		/*
		 * Create a request and the associate buffers
		 *
		 * ScreenPosition: This is the viewport coordinate in 2D of the pixel to analyze
		 * viewportUniqueId: The id of the view (FSceneView::State::GetViewKey) we want to capture the pixel, ScreenPosition has to come from this viewport
		 * SceneInterface: The interface to set the data for the next render frame.
		 * bInGameViewMode: Whether the next render frame is a game view.
		 * InPreExposure: The preexposure for the next render frame.
		*/
		 void CreatePixelInspectorRequest(FVector2D InspectViewportUV, int32 viewportUniqueId, FSceneInterface *SceneInterface, bool bInGameViewMode, float InPreExposure);

		 /**
		 * Look if there is some request ready to be read and retrieve the value.
		 * If there is a request that are ready it will read the gpu buffer to get the value
		 * and store the result.
		 * The request will be configure to be available again and the buffers will be release.
		 */
		 void ReadBackRequestData();

	protected:

		/** 
		 * Create the necessary rendertarget buffers for a request and set the render scene data.
		 * First created buffer (1x1) is for the normal (GBufferA) which can be of the following format: PF_FloatRGBA PF_B8G8R8A8 or PF_A2B10G10R10, depending on the precision settings
		 * Second created buffer (1x4) is for the other data (GBuffer B, C, D and E) which can be of the following format: PF_FloatRGBA or PF_B8G8R8A8, depending on the precision settings
		 *
		 * GBufferFormat: 0(Low RGB8), 1 (default), 5(float)
		 *
		 * Return a unique Index to allow the request to know how to find them in the FPixelInspectorData at the post process time when sending the read buffer graphic commands.
		 */
		int32 CreateRequestBuffer(FSceneInterface *SceneInterface, const int32 GBufferFormat, bool bInGameViewMode);

		/**
		 * Release all Ubuffer with the BufferIndex so the garbage collector will destroy them.
		 */
		void ReleaseBuffers(int32 BufferIndex);

		void OnLevelActorDeleted(AActor* Actor);
		
		void OnRedrawViewport(bool bInvalidateHitProxies);

		/* 
		* Use by the Coordinate mode only, this change the realtime state of the viewport if the state is not true
		*/
		void SetCurrentViewportInRealtime();

		void SetPixelInspectorState(bool bInIsPixelInspectorEnabled);

	private:

		void ReleaseAllRequests();
		FDelegateHandle OnLevelActorDeletedDelegateHandle;
		FDelegateHandle OnEditorCloseHandle;
		FDelegateHandle OnRedrawViewportHandle;
		FDelegateHandle OnApplicationPreInputKeyDownListenerHandle;

		bool bIsPixelInspectorEnable;

		int32 TickSinceLastCreateRequest;
		FPixelInspectorRequest Requests[2];

		//////////////////////////////////////////////////////////////////////////
		//Buffer management we can do only one pixel inspection per frame
		//We have two buffer of each type to not halt the render thread when we do the read back from the GPU
		//FinalColor Buffer
		UTextureRenderTarget2D* Buffer_FinalColor_AnyFormat[2];
		//Depth Buffer
		UTextureRenderTarget2D* Buffer_Depth_Float[2];
		//SceneColor Buffers
		UTextureRenderTarget2D* Buffer_SceneColorBeforePost_Float[2];
		UTextureRenderTarget2D* Buffer_SceneColorBeforeToneMap_Float[2];
		//GBufferA RenderTarget
		UTextureRenderTarget2D* Buffer_A_Float[2];
		UTextureRenderTarget2D* Buffer_A_RGB8[2];
		UTextureRenderTarget2D* Buffer_A_RGB10[2];
		//GBuffer BCDE RenderTarget
		UTextureRenderTarget2D* Buffer_BCDEF_Float[2];
		UTextureRenderTarget2D* Buffer_BCDEF_RGB8[2];
		//Which index we are at for the current Request
		int32 LastBufferIndex;

		//////////////////////////////////////////////////////////////////////////
		// ReadBack Data
		TArray<PixelInspectorResult> AccumulationResult;

		//////////////////////////////////////////////////////////////////////////
		// Display UObject to use the Detail Property Widget
		
		UPixelInspectorView *DisplayResult;
		
		FIntPoint LastViewportInspectionSize;
		
		FIntPoint LastViewportInspectionPosition;
		
		uint32 LastViewportId;

		TSharedPtr<IDetailsView> DisplayDetailsView;

		TSharedPtr< class FPixelInspectorSceneViewExtension, ESPMode::ThreadSafe > PixelInspectorSceneViewExtension;

};
}
