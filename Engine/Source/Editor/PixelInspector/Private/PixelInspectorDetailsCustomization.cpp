// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspectorDetailsCustomization.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PixelInspectorModule.h"
#include "PixelInspectorView.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;
class UObject;
struct FGeometry;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "PixelInspector"


FPixelInspectorDetailsCustomization::FPixelInspectorDetailsCustomization()
{
	CachedDetailBuilder = nullptr;
}

TSharedRef<IDetailCustomization> FPixelInspectorDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FPixelInspectorDetailsCustomization);
}

TSharedRef<SHorizontalBox> FPixelInspectorDetailsCustomization::GetGridColorContext()
{
	TSharedRef<SHorizontalBox> HorizontalMainGrid = SNew(SHorizontalBox);
	for (int32 ColumnIndex = 0; ColumnIndex < FinalColorContextGridSize; ++ColumnIndex)
	{
		TSharedRef<SVerticalBox> VerticalColumn = SNew(SVerticalBox);
		for (int32 RowIndex = 0; RowIndex < FinalColorContextGridSize; ++RowIndex)
		{
			VerticalColumn->AddSlot()
				.AutoHeight()
				.Padding(0.0f, 2.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					CreateColorCell(RowIndex, ColumnIndex, PixelInspectorView->FinalColorContext[ColumnIndex + RowIndex*FinalColorContextGridSize])
				];
		}
		HorizontalMainGrid->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 2.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				VerticalColumn
			];
	}
	return HorizontalMainGrid;
}

FReply FPixelInspectorDetailsCustomization::HandleColorCellMouseButtonDown(const FGeometry &, const FPointerEvent &, int32 RowIndex, int32 ColumnIndex)
{
	//Send a create a request to the new Coordinate
	if (RowIndex < 0 || RowIndex > FinalColorContextGridSize ||
		ColumnIndex < 0 || ColumnIndex > FinalColorContextGridSize)
		return FReply::Handled();

	int32 DeltaX = ColumnIndex - FMath::FloorToInt((float)FinalColorContextGridSize / 2.0f);
	int32 DeltaY = RowIndex - FMath::FloorToInt((float)FinalColorContextGridSize / 2.0f);
	//If user click on the middle do nothing
	if (DeltaX == 0 && DeltaY == 0)
		return FReply::Handled();
	//Get the PixelInspector module
	FPixelInspectorModule& PixelInspectorModule = FModuleManager::LoadModuleChecked<FPixelInspectorModule>(TEXT("PixelInspectorModule"));
	FIntPoint InspectViewportPos = FIntPoint(-1, -1);
	uint32 CoordinateViewportId = 0;
	PixelInspectorModule.GetCoordinatePosition(InspectViewportPos, CoordinateViewportId);
	if (InspectViewportPos == FIntPoint(-1, -1))
		return FReply::Handled();

	InspectViewportPos.X += DeltaX;
	InspectViewportPos.Y += DeltaY;
	if (InspectViewportPos.X < 0 || InspectViewportPos.Y < 0)
		return FReply::Handled();

	bool IsInspectorActive = PixelInspectorModule.IsPixelInspectorEnable();
	if (!IsInspectorActive)
	{
		PixelInspectorModule.ActivateCoordinateMode();
	}
	PixelInspectorModule.SetCoordinatePosition(InspectViewportPos, true);
	return FReply::Handled();
}

TSharedRef<SColorBlock> FPixelInspectorDetailsCustomization::CreateColorCell(int32 RowIndex, int32 ColumnIndex, const FLinearColor &CellColor)
{
	int32 SquareSize = FMath::FloorToInt(80.0f / (float)FinalColorContextGridSize);
	return SNew(SColorBlock)
		.Color(CellColor)
		.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
		.Size(FVector2D(SquareSize, SquareSize))
		.OnMouseButtonDown(this, &FPixelInspectorDetailsCustomization::HandleColorCellMouseButtonDown, RowIndex, ColumnIndex);
}

void FPixelInspectorDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedDetailBuilder = &DetailBuilder;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	PixelInspectorView = Cast<UPixelInspectorView>(EditingObjects[0].Get());

	IDetailCategoryBuilder& FinalColorCategory = DetailBuilder.EditCategory("FinalColor", FText::GetEmpty());
	if (PixelInspectorView.IsValid() )
	{
		FDetailWidgetRow& MergeRow = FinalColorCategory.AddCustomRow(LOCTEXT("FinalColorContextArray", "Context Color"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ContextColorRowTitle", "Context Colors"))
		];

		MergeRow.ValueContent()
		[
			GetGridColorContext()
		];
	}
	

	// Show only the option that go with the shading model
	TSharedRef<IPropertyHandle> SubSurfaceColorProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, SubSurfaceColor));
	TSharedRef<IPropertyHandle> SubSurfaceProfileProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, SubsurfaceProfile));
	TSharedRef<IPropertyHandle> OpacityProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, Opacity));
	TSharedRef<IPropertyHandle> ClearCoatProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, ClearCoat));
	TSharedRef<IPropertyHandle> ClearCoatRoughnessProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, ClearCoatRoughness));
	TSharedRef<IPropertyHandle> WorldNormalProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, WorldNormal));
	TSharedRef<IPropertyHandle> BackLitProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, BackLit));
	TSharedRef<IPropertyHandle> ClothProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, Cloth));
	TSharedRef<IPropertyHandle> EyeTangentProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, EyeTangent));
	TSharedRef<IPropertyHandle> IrisMaskProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, IrisMask));
	TSharedRef<IPropertyHandle> IrisDistanceProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPixelInspectorView, IrisDistance));

	EMaterialShadingModel MaterialShadingModel = PixelInspectorView->MaterialShadingModel;
	switch (MaterialShadingModel)
	{
		case MSM_DefaultLit:
		case MSM_SingleLayerWater:
		case MSM_ThinTranslucent:
		case MSM_Unlit:
		{
			DetailBuilder.HideProperty(SubSurfaceColorProp);
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(OpacityProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(ClothProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_Subsurface:
		case MSM_PreintegratedSkin:
		case MSM_TwoSidedFoliage:
		{
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(ClothProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_SubsurfaceProfile:
		{
			DetailBuilder.HideProperty(SubSurfaceColorProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(ClothProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_ClearCoat:
		{
			DetailBuilder.HideProperty(SubSurfaceColorProp);
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(OpacityProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(ClothProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_Hair:
		{
			DetailBuilder.HideProperty(SubSurfaceColorProp);
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(OpacityProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(ClothProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_Cloth:
		{
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(OpacityProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(EyeTangentProp);
			DetailBuilder.HideProperty(IrisMaskProp);
			DetailBuilder.HideProperty(IrisDistanceProp);
		}
		break;
		case MSM_Eye:
		{
			DetailBuilder.HideProperty(SubSurfaceColorProp);
			DetailBuilder.HideProperty(SubSurfaceProfileProp);
			DetailBuilder.HideProperty(OpacityProp);
			DetailBuilder.HideProperty(ClearCoatProp);
			DetailBuilder.HideProperty(ClearCoatRoughnessProp);
			DetailBuilder.HideProperty(WorldNormalProp);
			DetailBuilder.HideProperty(BackLitProp);
			DetailBuilder.HideProperty(ClothProp);
		}
		break;
	}
}

#undef LOCTEXT_NAMESPACE
