// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class SColorBlock;
class SHorizontalBox;
class UPixelInspectorView;
struct FGeometry;
struct FPointerEvent;

class FPixelInspectorDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

	TWeakObjectPtr<UPixelInspectorView> PixelInspectorView;		// The UI data object being customised
	IDetailLayoutBuilder* CachedDetailBuilder;	// The detail builder for this cusomtomisation


private:
	FReply HandleColorCellMouseButtonDown(const FGeometry &, const FPointerEvent &, int32 RowIndex, int32 ColumnIndex);

	TSharedRef<SHorizontalBox> GetGridColorContext();
	TSharedRef<SColorBlock> CreateColorCell(int32 RowIndex, int32 ColumnIndex, const FLinearColor &CellColor);

	/** Use MakeInstance to create an instance of this class */
	FPixelInspectorDetailsCustomization();
};
