// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspectorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "PixelInspectorView.h"
#include "PixelInspector.h"
#include "PropertyEditorModule.h"
#include "PixelInspectorStyle.h"
#include "PixelInspectorDetailsCustomization.h"

#include "Widgets/Docking/SDockTab.h"

void FPixelInspectorModule::StartupModule()
{
	HPixelInspectorWindow = nullptr;
	bHasRegisteredTabSpawners = false;
}

void FPixelInspectorModule::ShutdownModule()
{
	FPixelInspectorStyle::Shutdown();
}

TSharedRef<SWidget> FPixelInspectorModule::CreatePixelInspectorWidget()
{
	SAssignNew(HPixelInspectorWindow, PixelInspector::SPixelInspector);		
	return HPixelInspectorWindow->AsShared();
}

void FPixelInspectorModule::ActivateCoordinateMode()
{
	if (!HPixelInspectorWindow.IsValid())
		return;
	HPixelInspectorWindow->HandleTogglePixelInspectorEnableButton();
}

bool FPixelInspectorModule::IsPixelInspectorEnable()
{
	if (!HPixelInspectorWindow.IsValid())
		return false;
	return HPixelInspectorWindow->IsPixelInspectorEnable();
}

void FPixelInspectorModule::GetCoordinatePosition(FIntPoint &InspectViewportCoordinate, uint32 &InspectViewportId)
{
	InspectViewportCoordinate = FIntPoint(-1, -1);
	if (!HPixelInspectorWindow.IsValid())
		return;
	InspectViewportId = HPixelInspectorWindow->GetCurrentViewportId().GetValue();
	InspectViewportCoordinate = HPixelInspectorWindow->GetCurrentCoordinate();
}

void FPixelInspectorModule::SetCoordinatePosition(FIntPoint &InspectViewportCoordinate, bool ReleaseAllRequest)
{
	if (!HPixelInspectorWindow.IsValid())
		return;
	HPixelInspectorWindow->SetCurrentCoordinate(InspectViewportCoordinate, ReleaseAllRequest);
}

bool FPixelInspectorModule::GetViewportRealtime(int32 ViewportUid, bool IsCurrentlyRealtime, bool IsMouseInsideViewport)
{
	bool bContainsOriginalValue = OriginalViewportStates.Contains(ViewportUid);
	if (IsMouseInsideViewport)
	{
		bool bIsPixelInspectorEnable = IsPixelInspectorEnable();
		if (!bContainsOriginalValue && bIsPixelInspectorEnable)
		{
			OriginalViewportStates.Add(ViewportUid, IsCurrentlyRealtime);
		}
		else if (bContainsOriginalValue && !bIsPixelInspectorEnable) //User hit esc key
		{
			return OriginalViewportStates.FindAndRemoveChecked(ViewportUid);
		}
		return bIsPixelInspectorEnable ? true : IsCurrentlyRealtime;
	}
	else
	{
		if (bContainsOriginalValue)
		{
			return OriginalViewportStates.FindAndRemoveChecked(ViewportUid);
		}
	}
	return IsCurrentlyRealtime;
}

void FPixelInspectorModule::CreatePixelInspectorRequest(FVector2D InspectViewportUV, int32 ViewportUniqueId, FSceneInterface *SceneInterface, bool bInGameViewMode, float InPreExposure)
{
	if (HPixelInspectorWindow.IsValid() == false)
	{
		return;
	}
	HPixelInspectorWindow->CreatePixelInspectorRequest(InspectViewportUV, ViewportUniqueId, SceneInterface, bInGameViewMode, InPreExposure);
}

void FPixelInspectorModule::SetViewportInformation(int32 ViewportUniqueId, FIntPoint ViewportSize)
{
	if (HPixelInspectorWindow.IsValid() == false)
	{
		return;
	}
	HPixelInspectorWindow->SetViewportInformation(ViewportUniqueId, ViewportSize);
}

void FPixelInspectorModule::ReadBackSync()
{
	if (HPixelInspectorWindow.IsValid() == false)
	{
		return;
	}
	HPixelInspectorWindow->ReadBackRequestData();
}

void FPixelInspectorModule::OnTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	if (HPixelInspectorWindow.IsValid())
	{
		HPixelInspectorWindow->OnWindowClosed();
	}
}

void FPixelInspectorModule::RegisterTabSpawner(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
{
	LLM_SCOPE_BYNAME(TEXT("PixelInspector"));
	if (bHasRegisteredTabSpawners)
	{
		UnregisterTabSpawner();
	}

	bHasRegisteredTabSpawners = true;
	
	FPixelInspectorStyle::Initialize();

	//Register the UPixelInspectorView detail customization
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UPixelInspectorView::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPixelInspectorDetailsCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();
	}

	{
		FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("LevelEditorPixelInspector", FOnSpawnTab::CreateRaw(this, &FPixelInspectorModule::MakePixelInspectorTab))
			.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorPixelInspector", "Pixel Inspector"))
			.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorPixelInspectorTooltipText", "Open the viewport pixel inspector tool."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "PixelInspector.TabIcon"));

		if (WorkspaceGroup.IsValid())
		{
			SpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}
	}
}

void FPixelInspectorModule::UnregisterTabSpawner()
{
	bHasRegisteredTabSpawners = false;

	//Unregister the custom detail layout
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomClassLayout(UPixelInspectorView::StaticClass()->GetFName());

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("LevelEditorPixelInspector");
}

TSharedRef<SDockTab> FPixelInspectorModule::MakePixelInspectorTab(const FSpawnTabArgs&)
{
	TSharedRef<SDockTab> PixelInspectorTab = SNew(SDockTab)
	.TabRole(ETabRole::NomadTab);
	
	PixelInspectorTab->SetContent(CreatePixelInspectorWidget());
	PixelInspectorTab->SetOnTabClosed( SDockTab::FOnTabClosedCallback::CreateRaw( this, &FPixelInspectorModule::OnTabClosed ) );
	
	return PixelInspectorTab;
}

IMPLEMENT_MODULE(FPixelInspectorModule, PixelInspectorModule);
