// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspectorResult.h"

#include "ColorManagement/ColorSpace.h"
#include "HAL/PlatformCrt.h"
#include "Math/Float16.h"
#include "Math/Float16Color.h"
#include "PixelInspectorView.h"


#define LOCTEXT_NAMESPACE "PixelInspector"

namespace PixelInspector
{
	void PixelInspectorResult::DecodeFinalColor(TArray<FLinearColor>& BufferFinalColorValue, float InGamma, bool bHasAlphaChannel)
	{
		FinalColor.Empty();
		if (BufferFinalColorValue.Num() <= 0)
		{
			//Initialize to black
			for (int i = 0; i < FinalColorContextGridSize * FinalColorContextGridSize; ++i)
			{
				FinalColor.Add(FColor::Green);
			}
			return;
		}
		for (FLinearColor ReadBackColor : BufferFinalColorValue)
		{
			//Set the color in linear, with sign/abs functions to prevent NaNs on negative values.
			FLinearColor FinalColorTemp;
			FinalColorTemp.R = FMath::Sign(ReadBackColor.R) * FMath::Pow(FMath::Abs(ReadBackColor.R), InGamma);
			FinalColorTemp.G = FMath::Sign(ReadBackColor.G) * FMath::Pow(FMath::Abs(ReadBackColor.G), InGamma);
			FinalColorTemp.B = FMath::Sign(ReadBackColor.B) * FMath::Pow(FMath::Abs(ReadBackColor.B), InGamma);

			if (bHasAlphaChannel)
			{
				// invert alpha
				FinalColorTemp.A = 1. - FMath::Pow(ReadBackColor.A, InGamma);
			}
			else
			{
				FinalColorTemp.A = 1.;
			}
			FinalColor.Add(FinalColorTemp);		
		}
	}
	void PixelInspectorResult::DecodeSceneColorBeforePostProcessing(TArray<FLinearColor> &BufferSceneColorValue)
	{
		if (BufferSceneColorValue.Num() <= 0)
		{
			SceneColorBeforePostProcessing = FLinearColor::Transparent;
			return;
		}
		SceneColorBeforePostProcessing = BufferSceneColorValue[0] * OneOverPreExposure;
		SceneColorBeforePostProcessing.A = 1.0 - BufferSceneColorValue[0].A;
	}
	void PixelInspectorResult::DecodeDepth(TArray<FLinearColor> &BufferDepthValue)
	{
		if (BufferDepthValue.Num() <= 0)
		{
			Depth = 0.0f;
			WorldPosition = FVector(0.0f);
			return;
		}
		Depth = BufferDepthValue[0].R;
	}
	void PixelInspectorResult::DecodeSceneColorBeforeToneMap(TArray<FLinearColor>& BufferSceneColorValue, bool bHasAlphaChannel)
	{
		if (BufferSceneColorValue.Num() <= 0)
		{
			SceneColorBeforeTonemap = FLinearColor::Transparent;
			LuminanceBeforeTonemap = 0.0f;
			return;
		}
		
		SceneColorBeforeTonemap = BufferSceneColorValue[0] * OneOverPreExposure;

		if (bHasAlphaChannel)
		{
			// invert alpha
			SceneColorBeforeTonemap.A = 1. - BufferSceneColorValue[0].A;
		}
		else
		{
			SceneColorBeforeTonemap.A = 1.;
		}

		LuminanceBeforeTonemap = UE::Color::FColorSpace::GetWorking().GetLuminance(SceneColorBeforeTonemap);
	}

	void PixelInspectorResult::DecodeBufferData(TArray<FColor> &BufferAValue, TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting)
	{
		DecodeBufferA(BufferAValue);
		DecodeBufferBCDE(BufferBCDEValue, AllowStaticLighting);
	}

	void PixelInspectorResult::DecodeBufferData(TArray<FLinearColor> &BufferAValue, TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting)
	{
		DecodeBufferA(BufferAValue);
		DecodeBufferBCDE(BufferBCDEValue, AllowStaticLighting);
	}

	void PixelInspectorResult::DecodeBufferData(TArray<FFloat16Color> &BufferAValue, TArray<FFloat16Color> &BufferBCDEValue, bool AllowStaticLighting)
	{
		DecodeBufferA(BufferAValue);
		DecodeBufferBCDE(BufferBCDEValue, AllowStaticLighting);
	}

	void PixelInspectorResult::DecodeBufferA(TArray<FColor> &BufferAValue)
	{
		if (BufferAValue.Num() > 0)
		{
			Normal = DecodeNormalFromBuffer(ConvertLinearRGBToFloat(BufferAValue[0]));
			PerObjectGBufferData = (float)(BufferAValue[0].A) / 255.0f;
		}
	}
	
	void PixelInspectorResult::DecodeBufferA(TArray<FLinearColor> &BufferAValue)
	{
		if (BufferAValue.Num() > 0)
		{
			Normal = DecodeNormalFromBuffer(FVector(BufferAValue[0].R, BufferAValue[0].G, BufferAValue[0].B));
			PerObjectGBufferData = BufferAValue[0].A;
		}
	}

	void PixelInspectorResult::DecodeBufferA(TArray<FFloat16Color> &BufferAValue)
	{
		if (BufferAValue.Num() > 0)
		{
			Normal = DecodeNormalFromBuffer(FVector(BufferAValue[0].R.GetFloat(), BufferAValue[0].G.GetFloat(), BufferAValue[0].B.GetFloat()));
			PerObjectGBufferData = BufferAValue[0].A.GetFloat();
		}
	}

	void PixelInspectorResult::DecodeBufferBCDE(TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting)
	{
		if (BufferBCDEValue.Num() > 0)
		{
			FVector BufferBFloat = ConvertLinearRGBToFloat(BufferBCDEValue[0]);
			Metallic = BufferBFloat.X;
			Specular = BufferBFloat.Y;
			Roughness = BufferBFloat.Z;
			float EncodedChannel = (float)(BufferBCDEValue[0].A) / 255.0f;
			ShadingModel = DecodeShadingModel(EncodedChannel);
			SelectiveOutputMask = DecodeSelectiveOutputMask(EncodedChannel);
		}
		if (BufferBCDEValue.Num() > 1)
		{
			//Transform the base color in linear
			FColor BaseColorSRGB(BufferBCDEValue[1].R, BufferBCDEValue[1].G, BufferBCDEValue[1].B);
			BaseColor = FLinearColor(BaseColorSRGB);
			float EncodedChannel = (float)(BufferBCDEValue[1].A) / 255.0f;
			if (AllowStaticLighting)
			{
				IndirectIrradiance = DecodeIndirectIrradiance(EncodedChannel);
				AmbientOcclusion = 1.0f;
			}
			else
			{
				IndirectIrradiance = 1.0f;
				AmbientOcclusion = EncodedChannel;
			}
			
		}
		if (BufferBCDEValue.Num() > 2)
		{
			//Set the custom data
			DecodeCustomData(FVector4(ConvertLinearRGBToFloat(BufferBCDEValue[2]), (float)(BufferBCDEValue[2].A) / 255.0f));
		}
	}

	void PixelInspectorResult::DecodeBufferBCDE(TArray<FFloat16Color> &BufferBCDEValue, bool AllowStaticLighting)
	{
		if (BufferBCDEValue.Num() > 0)
		{
			FVector BufferBFloat = FVector(BufferBCDEValue[0].R.GetFloat(), BufferBCDEValue[0].G.GetFloat(), BufferBCDEValue[0].B.GetFloat());
			Metallic = BufferBFloat.X;
			Specular = BufferBFloat.Y;
			Roughness = BufferBFloat.Z;
			float EncodedChannel = BufferBCDEValue[0].A.GetFloat();
			ShadingModel = DecodeShadingModel(EncodedChannel);
			SelectiveOutputMask = DecodeSelectiveOutputMask(EncodedChannel);
		}
		if (BufferBCDEValue.Num() > 1)
		{
			//Transform the base color in linear
			BaseColor = FLinearColor(BufferBCDEValue[1].R.GetFloat(), BufferBCDEValue[1].G.GetFloat(), BufferBCDEValue[1].B.GetFloat());
			float EncodedChannel = BufferBCDEValue[1].A.GetFloat();
			if (AllowStaticLighting)
			{
				IndirectIrradiance = DecodeIndirectIrradiance(EncodedChannel);
				AmbientOcclusion = 1.0f;
			}
			else
			{
				IndirectIrradiance = 1.0f;
				AmbientOcclusion = EncodedChannel;
			}
		}
		if (BufferBCDEValue.Num() > 2)
		{
			//Set the custom data
			DecodeCustomData(FVector4(BufferBCDEValue[2].R.GetFloat(), BufferBCDEValue[2].G.GetFloat(), BufferBCDEValue[2].B.GetFloat(), BufferBCDEValue[2].A.GetFloat()));
		}
	}

	FVector4 PixelInspectorResult::ConvertLinearRGBAToFloat(FColor LinearRGBColor)
	{
		FVector VectorRGB = ConvertLinearRGBToFloat(LinearRGBColor.R, LinearRGBColor.G, LinearRGBColor.B);
		return FVector4(VectorRGB, (float)(LinearRGBColor.A) / 255.0f);
	}

	FVector PixelInspectorResult::ConvertLinearRGBToFloat(FColor LinearRGBColor)
	{
		return ConvertLinearRGBToFloat(LinearRGBColor.R, LinearRGBColor.G, LinearRGBColor.B);
	}

	FVector PixelInspectorResult::ConvertLinearRGBToFloat(uint8 Red, uint8 Green, uint8 Blue)
	{
		FVector ResultFloat;
		ResultFloat.X = (float)Red / 255.0f;
		ResultFloat.Y = (float)Green / 255.0f;
		ResultFloat.Z = (float)Blue / 255.0f;
		return ResultFloat;
	}

	FLinearColor PixelInspectorResult::DecodeSubSurfaceColor(FVector EncodeColor)
	{
		return FLinearColor(EncodeColor.X * EncodeColor.X, EncodeColor.Y * EncodeColor.Y, EncodeColor.Z * EncodeColor.Z);
	}

	FVector PixelInspectorResult::DecodeNormalFromBuffer(FVector NormalEncoded)
	{
		return (NormalEncoded * 2.0f) - FVector(1.0f);
	}

	EMaterialShadingModel PixelInspectorResult::DecodeShadingModel(float InPackedChannel)
	{
		int32 ShadingModelId = ((uint32)FMath::RoundToInt(InPackedChannel * (float)0xFF)) & PIXEL_INSPECTOR_SHADINGMODELID_MASK;
		switch (ShadingModelId)
		{
		case PIXEL_INSPECTOR_SHADINGMODELID_UNLIT:
			return EMaterialShadingModel::MSM_Unlit;
		case PIXEL_INSPECTOR_SHADINGMODELID_DEFAULT_LIT:
			return EMaterialShadingModel::MSM_DefaultLit;
		case PIXEL_INSPECTOR_SHADINGMODELID_SUBSURFACE:
			return EMaterialShadingModel::MSM_Subsurface;
		case PIXEL_INSPECTOR_SHADINGMODELID_PREINTEGRATED_SKIN:
			return EMaterialShadingModel::MSM_PreintegratedSkin;
		case PIXEL_INSPECTOR_SHADINGMODELID_CLEAR_COAT:
			return EMaterialShadingModel::MSM_ClearCoat;
		case PIXEL_INSPECTOR_SHADINGMODELID_SUBSURFACE_PROFILE:
			return EMaterialShadingModel::MSM_SubsurfaceProfile;
		case PIXEL_INSPECTOR_SHADINGMODELID_TWOSIDED_FOLIAGE:
			return EMaterialShadingModel::MSM_TwoSidedFoliage;
		case PIXEL_INSPECTOR_SHADINGMODELID_HAIR:
			return EMaterialShadingModel::MSM_Hair;
		case PIXEL_INSPECTOR_SHADINGMODELID_CLOTH:
			return EMaterialShadingModel::MSM_Cloth;
		case PIXEL_INSPECTOR_SHADINGMODELID_EYE:
			return EMaterialShadingModel::MSM_Eye;
		case PIXEL_INSPECTOR_SHADINGMODELID_SINGLELAYERWATER:
			return EMaterialShadingModel::MSM_SingleLayerWater;
		case PIXEL_INSPECTOR_SHADINGMODELID_THIN_TRANSLUCENT:
			return EMaterialShadingModel::MSM_ThinTranslucent;
		case PIXEL_INSPECTOR_SHADINGMODELID_SUBSTRATE:
			return EMaterialShadingModel::MSM_Strata;
		};
		return EMaterialShadingModel::MSM_DefaultLit;
	}

	uint32 PixelInspectorResult::DecodeSelectiveOutputMask(float InPackedChannel)
	{
		return ((uint32)FMath::RoundToInt(InPackedChannel * (float)0xFF)) & ~PIXEL_INSPECTOR_SHADINGMODELID_MASK;
	}

	float PixelInspectorResult::DecodeIndirectIrradiance(float IndirectIrradianceEncoded)
	{
		// LogL -> L
		float LogL = IndirectIrradianceEncoded;
		const float LogBlackPoint = 0.00390625;	// exp2(-8);
		return exp2(LogL * 16 - 8) - LogBlackPoint;		// 1 exp2, 1 smad, 1 ssub
	}

	FVector PixelInspectorResult::OctahedronToUnitVector(FVector2D Oct)
	{
		float ResultDot = FMath::Abs(Oct.X) + FMath::Abs(Oct.Y);

		FVector N = FVector(Oct.X, Oct.Y, 1 - ResultDot);
		if (N.Z < 0)
		{
			FVector2D N_yx_abs = FVector2D(FMath::Abs(N.Y), FMath::Abs(N.X));
			FVector2D Unit(1.0f, 1.0f);
			FVector2D NegativeUnit(-1.0f, -1.0f);
			FVector2D Temp_1 = FVector2D(Unit.X - N_yx_abs.X, Unit.Y - N_yx_abs.Y);
			FVector2D Temp_2 = (N.X > 0 && N.Y > 0) ? Unit : NegativeUnit;
			N.X = Temp_1.X * Temp_2.X;
			N.Y = Temp_1.Y * Temp_2.Y;
		}

		N.Normalize();
		return N;
	}

	void PixelInspectorResult::DecodeCustomData(FVector4 InCustomData)
	{
		switch (ShadingModel)
		{
		case EMaterialShadingModel::MSM_Unlit:
		case EMaterialShadingModel::MSM_DefaultLit:
		case EMaterialShadingModel::MSM_SingleLayerWater:
		case EMaterialShadingModel::MSM_ThinTranslucent:
		case EMaterialShadingModel::MSM_Strata:
		{
			SubSurfaceColor = FVector3f(0.0f);
			Opacity = 0.0f;
		}
		break;
		case EMaterialShadingModel::MSM_Subsurface:
		case EMaterialShadingModel::MSM_PreintegratedSkin:
		case EMaterialShadingModel::MSM_TwoSidedFoliage:
		{
			FVector EncodedSubSurfaceColor = FVector(InCustomData.X, InCustomData.Y, InCustomData.Z);
			SubSurfaceColor = DecodeSubSurfaceColor(EncodedSubSurfaceColor);
			Opacity = InCustomData.W;
		}
		break;
		case EMaterialShadingModel::MSM_SubsurfaceProfile:
		{
			SubsurfaceProfile = FVector(InCustomData.X, InCustomData.Y, InCustomData.Z);
		}
		break;
		case EMaterialShadingModel::MSM_ClearCoat:
		{
			ClearCoat = InCustomData.X;
			ClearCoatRoughness = InCustomData.Y;
		}
		break;
		case EMaterialShadingModel::MSM_Hair:
		{
			FVector2D MinusHalf(-0.5f, -0.5f);
			FVector2D Octahedron(InCustomData.X, InCustomData.Y);
			Octahedron = Octahedron + MinusHalf;
			Octahedron *= 2.0f;
			WorldNormal = OctahedronToUnitVector(Octahedron);
			BackLit = InCustomData.Z;
		}
		break;
		case EMaterialShadingModel::MSM_Cloth:
		{
			SubSurfaceColor = FVector3f(InCustomData.X, InCustomData.Y, InCustomData.Z);
			Cloth = InCustomData.W;
		}
		break;
		case EMaterialShadingModel::MSM_Eye:
		{
			EyeTangent = FVector(0.0f); //Eye tangent is not activate yet in the shader
			IrisMask = InCustomData.Z;
			IrisDistance = InCustomData.W;
		}
		break;
		};
	}
};

#undef LOCTEXT_NAMESPACE
