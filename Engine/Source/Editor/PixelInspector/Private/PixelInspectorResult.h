// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"

class FFloat16Color;

#define PIXEL_INSPECTOR_SHADINGMODELID_UNLIT 0
#define PIXEL_INSPECTOR_SHADINGMODELID_DEFAULT_LIT 1
#define PIXEL_INSPECTOR_SHADINGMODELID_SUBSURFACE 2
#define PIXEL_INSPECTOR_SHADINGMODELID_PREINTEGRATED_SKIN 3
#define PIXEL_INSPECTOR_SHADINGMODELID_CLEAR_COAT 4
#define PIXEL_INSPECTOR_SHADINGMODELID_SUBSURFACE_PROFILE 5
#define PIXEL_INSPECTOR_SHADINGMODELID_TWOSIDED_FOLIAGE 6
#define PIXEL_INSPECTOR_SHADINGMODELID_HAIR 7
#define PIXEL_INSPECTOR_SHADINGMODELID_CLOTH 8
#define PIXEL_INSPECTOR_SHADINGMODELID_EYE 9
#define PIXEL_INSPECTOR_SHADINGMODELID_SINGLELAYERWATER 10
#define PIXEL_INSPECTOR_SHADINGMODELID_THIN_TRANSLUCENT 11
#define PIXEL_INSPECTOR_SHADINGMODELID_SUBSTRATE 12
#define PIXEL_INSPECTOR_SHADINGMODELID_MASK 0xF

namespace PixelInspector
{
	class PixelInspectorResult
	{
	public:
		PixelInspectorResult()
		{
			ViewUniqueId = -1;
			ViewportUV = FVector2D(-1, -1);

			PreExposure = 1.f;

			Depth = 0.0f;
			WorldPosition = FVector(0.0f);

			LuminanceBeforeTonemap = 0.0f;

			Normal = FVector(0.0f);
			PerObjectGBufferData = 0.0f;
			Metallic = 0.0f;
			Specular = 0.0f;
			Roughness = 0.0f;
			ShadingModel = MSM_DefaultLit;
			SelectiveOutputMask = 0;
			BaseColor = FLinearColor::Black;
			IndirectIrradiance = 0.0f;
			AmbientOcclusion = 0.0f;

			//Custom Data
			SubSurfaceColor = FVector3f(0.0f);
			Opacity = 0.0f;
			ClearCoat = 0.0f;
			ClearCoatRoughness = 0.0f;
			WorldNormal = FVector(0.0f);
			BackLit = 0.0f;
			Cloth = 0.0f;
			EyeTangent = FVector(0.0f);
			IrisMask = 0.0f;
			IrisDistance = 0.0f;
		}
		// Data Identification
		int32 ViewUniqueId;
		FVector2D ViewportUV;


		//////////////////////////////////////////////////////////////////////////
		// PreExposure used to render this frame.
		float PreExposure;
		float OneOverPreExposure;

		//////////////////////////////////////////////////////////////////////////
		// Final color 3x3 grid
		TArray<FLinearColor> FinalColor;

		//////////////////////////////////////////////////////////////////////////
		// Scene color
		FLinearColor SceneColorBeforePostProcessing;
		FLinearColor SceneColorBeforeTonemap;
		float LuminanceBeforeTonemap;

		//////////////////////////////////////////////////////////////////////////
		// Depth and world position
		float Depth;
		FVector WorldPosition;

		//////////////////////////////////////////////////////////////////////////
		//Buffers value
		FVector Normal; //GBufferA RGB
		float PerObjectGBufferData; //GBufferA A
		float Metallic; //GBufferB R
		float Specular; //GBufferB G
		float Roughness; //GBufferB B
		EMaterialShadingModel ShadingModel; //GBufferB A encode
		int32 SelectiveOutputMask; //GBufferB A encode
		FLinearColor BaseColor; //GBufferC RGB
		
		//Irradiance and Ambient occlusion decoding
		float IndirectIrradiance; //GBufferC A encode only if static light is allow 1 otherwise
		float AmbientOcclusion; //GBufferC A if static light is not allow 1 otherwise

		//////////////////////////////////////////////////////////////////////////
		// Per shader model Data

		//MSM_Subsurface
		//MSM_PreintegratedSkin
		//MSM_TwoSidedFoliage
		FLinearColor SubSurfaceColor; // GBufferD RGB
		float Opacity; // GBufferD A

		//MSM_SubsurfaceProfile
		FVector SubsurfaceProfile; // GBufferD RGB

		//MSM_ClearCoat
		float ClearCoat; // GBufferD R
		float ClearCoatRoughness; // GBufferD G

		//MSM_Hair
		FVector WorldNormal;
		float BackLit;

		//MSM_Cloth
		float Cloth;

		//MSM_Eye
		FVector EyeTangent;
		float IrisMask;
		float IrisDistance;

		/** Decodes final color from HDR input. */
		void DecodeFinalColor(TArray<FLinearColor> &BufferFinalColorValue, float InGamma, bool bHasAlphaChannel);
		void DecodeSceneColorBeforePostProcessing(TArray<FLinearColor> &BufferSceneColorValue);
		void DecodeDepth(TArray<FLinearColor> &BufferDepthValue);
		void DecodeSceneColorBeforeToneMap(TArray<FLinearColor>& BufferSceneColorValue, bool bHasAlphaChannel);

		void DecodeBufferData(TArray<FColor> &BufferAValue, TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting);
		void DecodeBufferData(TArray<FLinearColor> &BufferAValue, TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting);
		void DecodeBufferData(TArray<FFloat16Color> &BufferAValue, TArray<FFloat16Color> &BufferBCDEValue, bool AllowStaticLighting);

	private:

		void DecodeBufferA(TArray<FColor> &BufferAValue);
		void DecodeBufferA(TArray<FLinearColor> &BufferAValue);
		void DecodeBufferA(TArray<FFloat16Color> &BufferAValue);

		void DecodeBufferBCDE(TArray<FColor> &BufferBCDEValue, bool AllowStaticLighting);
		void DecodeBufferBCDE(TArray<FFloat16Color> &BufferBCDEValue, bool AllowStaticLighting);

		FVector4 ConvertLinearRGBAToFloat(FColor LinearRGBColor);
		FVector ConvertLinearRGBToFloat(FColor LinearRGBColor);
		FVector ConvertLinearRGBToFloat(uint8 Red, uint8 Green, uint8 Blue);
		FLinearColor DecodeSubSurfaceColor(FVector EncodeColor);
		FVector DecodeNormalFromBuffer(FVector NormalEncoded);
		EMaterialShadingModel DecodeShadingModel(float InPackedChannel);
		uint32 DecodeSelectiveOutputMask(float InPackedChannel);
		float DecodeIndirectIrradiance(float IndirectIrradiance);
		FVector OctahedronToUnitVector(FVector2D Oct);
		void DecodeCustomData(FVector4 InCustomData);
	};
};
