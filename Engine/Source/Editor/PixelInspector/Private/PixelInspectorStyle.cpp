// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspectorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BORDER_BRUSH(RelativePath, ...) FSlateBorderBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

TSharedPtr< FSlateStyleSet > FPixelInspectorStyle::StyleSet = NULL;
TSharedPtr< class ISlateStyle > FPixelInspectorStyle::Get() { return StyleSet; }

FName FPixelInspectorStyle::GetStyleSetName()
{
	static FName PixelInspectorStyleName(TEXT("PixelInspectorStyle"));
	return PixelInspectorStyleName;
}

void FPixelInspectorStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon9x19(9.0f, 19.0f);
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon27x31(27.0f, 31.0f);
	const FVector2D Icon26x26(26.0f, 26.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon75x82(75.0f, 82.0f);
	const FVector2D Icon360x32(360.0f, 32.0f);
	const FVector2D Icon171x39(171.0f, 39.0f);
	const FVector2D Icon170x50(170.0f, 50.0f);
	const FVector2D Icon267x140(170.0f, 50.0f);

	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable( new FSlateStyleSet(FPixelInspectorStyle::GetStyleSetName()) );
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// PixelInspector Icons
	{
		StyleSet->Set( "PixelInspector.TabIcon", new IMAGE_BRUSH( "Icons/PixelInspector/icon_PixelInspector_tab_16x", Icon16x16 ) );
		StyleSet->Set( "PixelInspector.Enabled", new IMAGE_BRUSH( "Icons/PixelInspector/icon_PixelInspector_Stop_16x", Icon16x16 ) );
		StyleSet->Set( "PixelInspector.Disabled", new IMAGE_BRUSH( "Icons/PixelInspector/icon_PixelInspector_Start_16x", Icon16x16 ) );
	}

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
};

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH

void FPixelInspectorStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}
