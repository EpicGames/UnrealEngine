// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelInspectorView.h"

#include "Containers/Array.h"
#include "PixelInspectorResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelInspectorView)


UPixelInspectorView::UPixelInspectorView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	for (int i = 0; i < FinalColorContextGridSize*FinalColorContextGridSize; ++i)
	{
		FinalColorContext[i] = FLinearColor::Green;
	}
	FinalColor = FLinearColor::Green;
	PreExposure = 1.f;
	SceneColorBeforePostProcessing = FLinearColor::Green;
	SceneColorBeforeTonemap = FLinearColor::Black;
	LuminanceBeforeTonemap = 0.0f;
	Normal = FVector(0.0f);
	PerObjectGBufferData = 0.0f;
	Metallic = 0.0f;
	Specular = 0.0f;
	Roughness = 0.0f;
	MaterialShadingModel = EMaterialShadingModel::MSM_DefaultLit;
	SelectiveOutputMask = 0;
	BaseColor = FLinearColor::Black;
	IndirectIrradiance = 0.0f;
	AmbientOcclusion = 0.0f;

	//Custom Data
	SubSurfaceColor = FLinearColor::Black;
	SubsurfaceProfile = FVector(0.0f);
	Opacity = 0.0f;
	ClearCoat = 0.0f;
	ClearCoatRoughness = 0.0f;
	WorldNormal = FVector(0.0f);
	BackLit = 0.0f;
	Cloth = 0.0f;
	EyeTangent = FVector(0.0f);
	IrisMask = 0.0f;
	IrisDistance = 0.0f;
}

void UPixelInspectorView::SetFromResult(PixelInspector::PixelInspectorResult &Result)
{
	FinalColor = FLinearColor::Green;
	//Take the center of the array, TODO display the context
	for (int i = 0; i < FinalColorContextGridSize*FinalColorContextGridSize; ++i)
	{
		if (Result.FinalColor.Num() > i)
		{
			FinalColorContext[i] = Result.FinalColor[i];
			if (i == FMath::FloorToInt((FinalColorContextGridSize*FinalColorContextGridSize)/2.0f))
			{
				FinalColor = Result.FinalColor[i];
			}
		}
		else
		{
			FinalColorContext[i] = FLinearColor::Green;
		}
	}
	
	PreExposure = Result.PreExposure;
	SceneColorBeforePostProcessing = Result.SceneColorBeforePostProcessing;
	SceneColorBeforeTonemap = Result.SceneColorBeforeTonemap;
	LuminanceBeforeTonemap = Result.LuminanceBeforeTonemap;
	Normal = Result.Normal;
	PerObjectGBufferData = Result.PerObjectGBufferData;
	Metallic = Result.Metallic;
	Specular = Result.Specular;
	Roughness = Result.Roughness;
	MaterialShadingModel = Result.ShadingModel;
	SelectiveOutputMask = Result.SelectiveOutputMask;
	BaseColor = Result.BaseColor;
	IndirectIrradiance = Result.IndirectIrradiance;
	AmbientOcclusion = Result.AmbientOcclusion;

	//Custom Data
	SubSurfaceColor = Result.SubSurfaceColor;
	SubsurfaceProfile = Result.SubsurfaceProfile;
	Opacity = Result.Opacity;
	ClearCoat = Result.ClearCoat;
	ClearCoatRoughness = Result.ClearCoatRoughness;
	WorldNormal = Result.WorldNormal;
	BackLit = Result.BackLit;
	Cloth = Result.Cloth;
	EyeTangent = Result.EyeTangent;
	IrisMask = Result.IrisMask;
	IrisDistance = Result.IrisDistance;
}
