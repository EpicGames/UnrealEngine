// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Engine/EngineTypes.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "PixelInspectorView.generated.h"

namespace PixelInspector { class PixelInspectorResult; }

#define FinalColorContextGridSize 7

// Note: UPixelInspectorView is only ever created as a transient object, so making properties transient is redundant.

UCLASS(HideCategories = Object, MinimalAPI)
class UPixelInspectorView : public UObject
{
	GENERATED_UCLASS_BODY()

	FLinearColor FinalColorContext[FinalColorContextGridSize*FinalColorContextGridSize];

	/** Final color in display space after tone mapping. */
	UPROPERTY(VisibleAnywhere, category = FinalColor)
	FLinearColor FinalColor;

	/** Amount of exposure applied to Scene Color due to auto-exposure and/or exposure compensation. */
	UPROPERTY(VisibleAnywhere, category = SceneColor)
	float PreExposure;

	/** HDR scene-linear color in working color space, before post-processing. Previously called Scene Color. */
	UPROPERTY(VisibleAnywhere, category = SceneColor, meta = (DisplayName = "Scene Color Before Post-Processing"))
	FLinearColor SceneColorBeforePostProcessing;

	/** HDR scene-linear color in working color space, before tone mapping. Previously called Hdr Color. */
	UPROPERTY(VisibleAnywhere, category = SceneColor)
	FLinearColor SceneColorBeforeTonemap;

	/** Luminance of the Scene Color Before Tonemap. */
	UPROPERTY(VisibleAnywhere, category = SceneColor)
	float LuminanceBeforeTonemap;

	/** From the GBufferA RGB Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferA)
	FVector Normal;

	/** From the GBufferA A Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferA)
	float PerObjectGBufferData;

	/** From the GBufferB R Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferB)
	float Metallic;

	/** From the GBufferB G Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferB)
	float Specular;

	/** From the GBufferB B Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferB)
	float Roughness;

	/** From the GBufferB A Channel encoded with SelectiveOutputMask. */
	UPROPERTY(VisibleAnywhere, category = GBufferB)
	TEnumAsByte<enum EMaterialShadingModel> MaterialShadingModel;

	/** From the GBufferB A Channel encoded with ShadingModel. */
	UPROPERTY(VisibleAnywhere, category = GBufferB)
	int32 SelectiveOutputMask;

	/** From the GBufferC RGB Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferC)
	FLinearColor BaseColor;

	/** From the GBufferC A Channel encoded with AmbientOcclusion. */
	UPROPERTY(VisibleAnywhere, category = GBufferC)
	float IndirectIrradiance;

	/** From the GBufferC A Channel encoded with IndirectIrradiance. */
	UPROPERTY(VisibleAnywhere, category = GBufferC)
	float AmbientOcclusion;

	//Custom Data section

	/** From the GBufferD RGB Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	FLinearColor SubSurfaceColor;

	/** From the GBufferD RGB Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	FVector SubsurfaceProfile;

	/** From the GBufferD A Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float Opacity;

	/** From the GBufferD R Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float ClearCoat;

	/** From the GBufferD G Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float ClearCoatRoughness;

	/** From the GBufferD RG Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	FVector WorldNormal;

	/** From the GBufferD B Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float BackLit;

	/** From the GBufferD A Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float Cloth;

	/** From the GBufferD RG Channels. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	FVector EyeTangent;

	/** From the GBufferD B Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float IrisMask;

	/** From the GBufferD A Channel. */
	UPROPERTY(VisibleAnywhere, category = GBufferD)
	float IrisDistance;

	void SetFromResult(PixelInspector::PixelInspectorResult &Result);
	/*
	//////////////////////////////////////////////////////////////////////////
	// Per shader model Data

	//MSM_Subsurface
	//MSM_PreintegratedSkin
	//MSM_SubsurfaceProfile
	//MSM_TwoSidedFoliage
	FVector SubSurfaceColor; // GBufferD RGB
	float Opacity; // GBufferD A

	//MSM_ClearCoat
	float ClearCoat; // GBufferD R
	float ClearCoatRoughness; // GBufferD G

	//MSM_Hair
	FVector WorldNormal; // GBufferD RG
	float BackLit; // GBufferD B

	//MSM_Cloth, Use also the sub surface color
	float Cloth; // GBufferD A

	//MSM_Eye
	FVector Tangent; // GBufferD RG
	float IrisMask; // GBufferD B
	float IrisDistance; // GBufferD A
	*/
};



