// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"

class FSceneInterface;
class FSpawnTabArgs;
class FWorkspaceItem;

namespace PixelInspector { class SPixelInspector; };

/**
* The module holding all of the UI related pieces for pixel inspector
*/
class FPixelInspectorModule : public IModuleInterface
{

public:
	/**
	* Called right after the module DLL has been loaded and the module object has been created
	*/
	virtual void StartupModule();

	/**
	* Called before the module is unloaded, right before the module object is destroyed.
	*/
	virtual void ShutdownModule();

	/** Creates the HLOD Outliner widget */
	virtual TSharedRef<SWidget> CreatePixelInspectorWidget();

	virtual void ActivateCoordinateMode();

	virtual bool IsPixelInspectorEnable();

	virtual void GetCoordinatePosition(FIntPoint &InspectViewportCoordinate, uint32 &InspectViewportId);

	virtual void SetCoordinatePosition(FIntPoint &InspectViewportCoordinate, bool ReleaseAllRequest);

	virtual bool GetViewportRealtime(int32 ViewportUid, bool IsCurrentlyRealtime, bool IsMouseInsideViewport);

	virtual void CreatePixelInspectorRequest(FVector2D InspectViewportUV, int32 ViewportUniqueId, FSceneInterface *SceneInterface, bool bInGameViewMode, float InPreExposure);

	virtual void SetViewportInformation(int32 ViewportUniqueId, FIntPoint ViewportSize);

	virtual void ReadBackSync();

	virtual void RegisterTabSpawner(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup);

	virtual void UnregisterTabSpawner();
	
private:
	
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

private:
	TSharedRef<SDockTab> MakePixelInspectorTab(const FSpawnTabArgs&);

	bool bHasRegisteredTabSpawners;
	TSharedPtr<PixelInspector::SPixelInspector> HPixelInspectorWindow;

	TMap<uint32, bool> OriginalViewportStates;
};
